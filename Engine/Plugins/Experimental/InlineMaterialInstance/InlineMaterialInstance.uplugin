{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Inline Material Instance",
	"Description": "Allows creation and modification of materials within the property editor",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "InlineMaterialInstanceEditor",
			"Type": "Editor",
			"LoadingPhase": "PostDefault"
		}
	]
}
