// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class InlineMaterialInstanceEditor : ModuleRules
{
	public InlineMaterialInstanceEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"Landscape",
				"PropertyEditor",
				"MaterialEditor",
				"Slate",
				"SlateCore",
				"UnrealEd",
            }
		);
	}
}
