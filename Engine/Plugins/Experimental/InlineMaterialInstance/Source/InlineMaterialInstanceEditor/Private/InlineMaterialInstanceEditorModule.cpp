// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/LowLevelMemTracker.h"
#include "MaterialList.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SMaterialDynamicWidgets.h"

LLM_DEFINE_TAG(InlineMaterialInstanceEditor);

#define LOCTEXT_NAMESPACE "FInlineMaterialInstanceEditorModule"

class FInlineMaterialInstanceEditorModule : public IModuleInterface
{
public:
	void StartupModule()
	{
		LLM_SCOPE_BYTAG(InlineMaterialInstanceEditor);

		// Add bottom extender for material item
		FMaterialList::OnAddMaterialItemViewExtraBottomWidget.AddLambda([](const TSharedRef<FMaterialItemView>& InMaterialItemView, UActorComponent* InCurrentComponent, IDetailLayoutBuilder& InDetailBuilder, TArray<TSharedPtr<SWidget>>& OutExtensions)
		{
			LLM_SCOPE_BYTAG(InlineMaterialInstanceEditor);
			OutExtensions.Add(SNew(SMaterialDynamicView, InMaterialItemView, InCurrentComponent));
		});
	}

	void ShutdownModule()
	{
		LLM_SCOPE_BYTAG(InlineMaterialInstanceEditor);
		FMaterialList::OnAddMaterialItemViewExtraBottomWidget.RemoveAll(this);
	}
};

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FInlineMaterialInstanceEditorModule, InlineMaterialInstanceEditor)
