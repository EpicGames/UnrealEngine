// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMaterialDynamicParametersPanelWidget.h"

#include "DetailColumnSizeData.h"
#include "DetailWidgetRow.h"
#include "Styling/AppStyle.h"
#include "IDetailPropertyRow.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Materials/MaterialInstance.h"
#include "Modules/ModuleManager.h"
#include "UObject/Object.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "MaterialDynamicParametersPanelWidget"

enum EStackDataType
{
	Stack,
	Asset,
	Group,
	Property,
	PropertyChild,
};

struct FPropertySortedParamData
{
	EStackDataType StackDataType;

	FMaterialParameterInfo ParameterInfo;

	TSharedPtr<IDetailTreeNode> ParameterNode;

	TSharedPtr<IPropertyHandle> ParameterHandle;

	TArray<TSharedPtr<FPropertySortedParamData>> Children;

	FString NodeKey;

	FName GroupName;
};

struct FPropertyUnsortedParamData
{
	TSharedPtr<IDetailTreeNode> ParameterNode;

	TSharedPtr<IPropertyHandle> ParameterHandle;

	FMaterialParameterInfo ParameterInfo;

	FName GroupName;
};

namespace Utilities
{
	template<typename ParameterValueType>
	void AddParameters(TSharedPtr<IPropertyHandle> InPropertyHandle, const FName InParameterName, TArray<TSharedPtr<IPropertyHandle>>& OutDeferredSearches, TArray<FPropertyUnsortedParamData>& OutUnsortedParameters)
	{
		if (!InPropertyHandle.IsValid())
		{
			return;
		}
		
		TArray<void*> GroupPtrs;
		InPropertyHandle->AccessRawData(GroupPtrs);
		const auto GroupIt = GroupPtrs.CreateConstIterator();
		TArray<ParameterValueType>* ParameterGroupArrayPtr = reinterpret_cast<TArray<ParameterValueType>*>(*GroupIt);
		if (!ParameterGroupArrayPtr)
		{
			return;
		}

		const TSharedPtr<IPropertyHandleArray> ParameterValuesArray = InPropertyHandle->AsArray();

		uint32 NumElements = 0;
		ParameterValuesArray->GetNumElements(NumElements);
		for (uint32 Index = 0; Index < NumElements; ++Index)
		{
			const ParameterValueType& ParameterValue = (*ParameterGroupArrayPtr)[Index];
			if (!ParameterValue.ExpressionGUID.IsValid())
			{
				continue;
			}

			if (const TSharedPtr<IPropertyHandle> ElementPropertyHandle = ParameterValuesArray->GetElement(Index))
			{
				FPropertyUnsortedParamData NonLayerProperty;
				TSharedPtr<IPropertyHandle> ParameterValuePropertyHandle = ElementPropertyHandle->GetChildHandle(InParameterName);

				NonLayerProperty.ParameterInfo = ParameterValue.ParameterInfo;
				NonLayerProperty.GroupName = TEXT("Group");

				OutDeferredSearches.Add(ParameterValuePropertyHandle);
				OutUnsortedParameters.Add(NonLayerProperty);
			}
		}
	}
};

// ********* SMaterialParametersOverviewTree *******
class SMaterialDynamicParametersOverviewTree : public STreeView<TSharedPtr<FPropertySortedParamData>>
{
	friend class SMaterialDynamicParametersOverviewTreeItem;

public:

	SLATE_BEGIN_ARGS(SMaterialDynamicParametersOverviewTree)
	{}

	SLATE_ARGUMENT(UMaterialInstance*, InMaterialInstance)
	SLATE_ARGUMENT(TSharedPtr<SMaterialDynamicParametersPanelWidget>, InOwner)
	SLATE_ARGUMENT(TSharedPtr<SScrollBar>, InScrollbar)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);
	TSharedRef< ITableRow > OnGenerateRowMaterialLayersFunctionsTreeView(TSharedPtr<FPropertySortedParamData> Item, const TSharedRef< STableViewBase >& OwnerTable);
	void OnGetChildrenMaterialLayersFunctionsTreeView(TSharedPtr<FPropertySortedParamData> InParent, TArray< TSharedPtr<FPropertySortedParamData> >& OutChildren);
	void OnExpansionChanged(TSharedPtr<FPropertySortedParamData> Item, bool bIsExpanded);
	void SetParentsExpansionState();

	/** Builds the custom parameter groups category */
	void CreateGroupsWidget();

	TWeakPtr<SMaterialDynamicParametersPanelWidget> GetOwner() { return Owner; }
	bool HasAnyParameters() const { return bHasAnyParameters; }

	FDetailColumnSizeData& GetColumnSizeData() { return ColumnSizeData; }

protected:

	void ShowSubParameters();

public:
	/** Object that stores all of the possible parameters we can edit */
	TWeakObjectPtr<UMaterialInstance> MaterialInstance;

private:
	TArray<TSharedPtr<FPropertySortedParamData>> SortedParameters;

	TArray<FPropertyUnsortedParamData> UnsortedParameters;

	TWeakPtr<SMaterialDynamicParametersPanelWidget> Owner;

	bool bHasAnyParameters;
	
	FDetailColumnSizeData ColumnSizeData;
};

// ********* SMaterialParametersOverviewTreeItem *******
class SMaterialDynamicParametersOverviewTreeItem : public STableRow< TSharedPtr<FPropertySortedParamData> >
{
public:

	SLATE_BEGIN_ARGS(SMaterialDynamicParametersOverviewTreeItem)
		: _StackParameterData(nullptr),
		_MaterialInstance(nullptr)
	{}

	/** The item content. */
	SLATE_ARGUMENT(TSharedPtr<FPropertySortedParamData>, StackParameterData)
	SLATE_ARGUMENT(UMaterialInstance*, MaterialInstance)
	SLATE_ARGUMENT(TSharedPtr<SMaterialDynamicParametersOverviewTree>, InTree)
	SLATE_END_ARGS()

	void RefreshOnRowChange(const FAssetData& AssetData, TSharedPtr<SMaterialDynamicParametersOverviewTree> InTree);

	/**
	* Construct the widget
	*
	* @param InArgs   A declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

private:
	const FSlateBrush* GetBorderImage() const;

	FSlateColor GetOuterBackgroundColor(TSharedPtr<FPropertySortedParamData> InParamData) const;

	TSharedRef<SWidget> GetRowExtensionButtons(TSharedPtr<IPropertyHandle> InPropertyHandle);
private:

	/** The node info to build the tree view row from. */
	TSharedPtr<FPropertySortedParamData> StackParameterData;

	/** The tree that contains this item */
	TWeakPtr<SMaterialDynamicParametersOverviewTree> Tree;

	/** The set of material parameters this is associated with */
	TWeakObjectPtr<UMaterialInstance> MaterialInstance;

	/** Pointer to copied ValuePtr. Needed for workaround for now */
	TUniquePtr<uint8> NewDefaultValuePtr;

	/** Pointer to Widget for Workaround. */
	TSharedPtr<SWidget> ResetArrow;
};



const FSlateBrush* SMaterialDynamicParametersOverviewTreeItem::GetBorderImage() const
{
	return FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
}

FSlateColor SMaterialDynamicParametersOverviewTreeItem::GetOuterBackgroundColor(TSharedPtr<FPropertySortedParamData> InParamData) const
{
	if (IsHovered() || InParamData->StackDataType == EStackDataType::Group)
	{
		return FAppStyle::Get().GetSlateColor("Colors.Header");
	}

	return FAppStyle::Get().GetSlateColor("Colors.Panel");
}

TSharedRef<SWidget> SMaterialDynamicParametersOverviewTreeItem::GetRowExtensionButtons(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	if (!InPropertyHandle.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	const auto ResetDelegate = FExecuteAction::CreateLambda([this, InPropertyHandle]()
	{
		if (InPropertyHandle.IsValid())
		{
			InPropertyHandle->ResetToDefault();
			ResetArrow->SetVisibility(EVisibility::Hidden);
		}
	});

	/** Needed as Dynamic Material Instances don't have a default to use normally for DiffersFromDefault */
	const auto DiffersFromDefaultDelegate = FIsActionButtonVisible::CreateLambda([this, InPropertyHandle]()
	{
		if (!NewDefaultValuePtr)
		{
			return true;
		}

		FProperty* Property = InPropertyHandle->GetProperty();
		
		void* ValuePtr;
		InPropertyHandle->GetValueData(ValuePtr);
		
		if (Property->Identical(NewDefaultValuePtr.Get(), ValuePtr))
		{
			return false;
		}
		return true;
	});
	
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	TArray<FPropertyRowExtensionButton> ExtensionButtons;
	FPropertyRowExtensionButton& ResetToDefault = ExtensionButtons.AddDefaulted_GetRef();
	ResetToDefault.Label = NSLOCTEXT("PropertyEditor", "ResetToDefault", "Reset to Default");
	ResetToDefault.UIAction = FUIAction(ResetDelegate, FCanExecuteAction(), FIsActionChecked(), DiffersFromDefaultDelegate);

	ResetToDefault.Icon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "PropertyWindow.DiffersFromDefault");
	ResetToDefault.ToolTip = NSLOCTEXT("PropertyEditor", "ResetToDefaultToolTip", "Reset this property to its default value.");

	// Add custom property extensions
	FOnGenerateGlobalRowExtensionArgs RowExtensionArgs;
	RowExtensionArgs.PropertyHandle = InPropertyHandle;
	PropertyEditorModule.GetGlobalRowExtensionDelegate().Broadcast(RowExtensionArgs, ExtensionButtons);
	
	// Build extension toolbar 
	FSlimHorizontalToolBarBuilder ToolbarBuilder(TSharedPtr<FUICommandList>(), FMultiBoxCustomization::None);
	ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);
	ToolbarBuilder.SetStyle(&FAppStyle::Get(), "DetailsView.ExtensionToolBar");
	for (const FPropertyRowExtensionButton& Extension : ExtensionButtons)
	{
		ToolbarBuilder.AddToolBarButton(Extension.UIAction, NAME_None, Extension.Label, Extension.ToolTip, Extension.Icon);
	}

	/** Set up the Default Value for the Workaround. */
	FProperty* Property = InPropertyHandle->GetProperty();
	NewDefaultValuePtr = MakeUnique<uint8>(Property->GetSize());
	void* ValuePtr;
	InPropertyHandle->GetValueData(ValuePtr);
	Property->CopyCompleteValue(NewDefaultValuePtr.Get(), ValuePtr);
	Property->ClearValue(NewDefaultValuePtr.Get());
	
	ResetArrow = ToolbarBuilder.MakeWidget();
	
	return ResetArrow.ToSharedRef();
}

void SMaterialDynamicParametersOverviewTreeItem::RefreshOnRowChange(const FAssetData& AssetData, TSharedPtr<SMaterialDynamicParametersOverviewTree> InTree)
{
	if (InTree.IsValid())
	{
		InTree->CreateGroupsWidget();
	}
}

void SMaterialDynamicParametersOverviewTreeItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	StackParameterData = InArgs._StackParameterData;
	MaterialInstance = InArgs._MaterialInstance;
	Tree = InArgs._InTree;

	TSharedRef<SWidget> LeftSideWidget = SNullWidget::NullWidget;
	TSharedRef<SWidget> RightSideWidget = SNullWidget::NullWidget;
	FText NameOverride;
	TSharedRef<SVerticalBox> WrapperWidget = SNew(SVerticalBox);

	auto GenerateLeftAndRightWidgets = [&](IDetailTreeNode& InNode)
	{
		const FNodeWidgets NodeWidgets = InNode.CreateNodeWidgets();
		if (NodeWidgets.NameWidget.IsValid() && NodeWidgets.ValueWidget.IsValid())
		{
			LeftSideWidget = NodeWidgets.NameWidget.ToSharedRef();
			RightSideWidget = SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					[
						NodeWidgets.ValueWidget.ToSharedRef()
					];
		}
	};

// GROUP --------------------------------------------------
	if (StackParameterData->StackDataType == EStackDataType::Group)
	{
		NameOverride = FText::FromName(StackParameterData->GroupName);
		LeftSideWidget = SNew(STextBlock)
			.TransformPolicy(ETextTransformPolicy::ToUpper)
			.Text(NameOverride)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.BoldFont"))
			.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle");
	}
// END GROUP

// PROPERTY ----------------------------------------------
	bool bIsPaddedProperty = false;
	if (StackParameterData->StackDataType == EStackDataType::Property)
	{
		NameOverride = FText::FromName(StackParameterData->ParameterInfo.Name);

		IDetailTreeNode& Node = *StackParameterData->ParameterNode;
		TSharedPtr<IDetailPropertyRow> GeneratedRow = StaticCastSharedPtr<IDetailPropertyRow>(Node.GetRow());
		IDetailPropertyRow& Row = *GeneratedRow.Get();
		Row.DisplayName(NameOverride);


		if (TSharedPtr<IPropertyHandle> PropertyHandle = StackParameterData->ParameterNode->CreatePropertyHandle())
		{
			PropertyHandle->MarkResetToDefaultCustomized(true);
		}

		FDetailWidgetDecl* CustomNameWidget = Row.CustomNameWidget();
		if (CustomNameWidget)
		{
			(*CustomNameWidget)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(NameOverride)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				]
			];
		}

		bIsPaddedProperty = true;

		GenerateLeftAndRightWidgets(Node);
	}
// END PROPERTY

// PROPERTY CHILD ----------------------------------------
	if (StackParameterData->StackDataType == EStackDataType::PropertyChild)
	{
		GenerateLeftAndRightWidgets(*StackParameterData->ParameterNode);
	}
// END PROPERTY CHILD

// FINAL WRAPPER

	{
		FDetailColumnSizeData& ColumnSizeData = InArgs._InTree->GetColumnSizeData();

		float ValuePadding = bIsPaddedProperty ? 20.0f : 0.0f;
		WrapperWidget->AddSlot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
				.Padding(FMargin(0, 0, 0, 1))
				[
					SNew(SBorder)
					.Padding(3.0f)
					.BorderImage(this, &SMaterialDynamicParametersOverviewTreeItem::GetBorderImage)
					.BorderBackgroundColor(this, &SMaterialDynamicParametersOverviewTreeItem::GetOuterBackgroundColor, StackParameterData)
					[
						SNew(SSplitter)
						.Style(FAppStyle::Get(), "DetailsView.Splitter")
						.PhysicalSplitterHandleSize(1.0f)
						.HitDetectionSplitterHandleSize(5.0f)
						+ SSplitter::Slot()
						.Value(ColumnSizeData.GetNameColumnWidth())
						.OnSlotResized(ColumnSizeData.GetOnNameColumnResized())
						.Value(0.25f)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(FMargin(3.0f))
							[
								SNew(SExpanderArrow, SharedThis(this))
							]
							+ SHorizontalBox::Slot()
							.Padding(FMargin(2.0f))
							.VAlign(VAlign_Center)
							[
								LeftSideWidget
							]
						]
						+ SSplitter::Slot()
						.Value(ColumnSizeData.GetValueColumnWidth())
						.OnSlotResized(ColumnSizeData.GetOnValueColumnResized()) 
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.Padding(FMargin(5.0f, 2.0f, 5.0, 2.0f))
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								RightSideWidget
							]
						]
					]
				]
			];
	}

	ChildSlot
		[
			WrapperWidget
		];


	STableRow< TSharedPtr<FPropertySortedParamData> >::ConstructInternal(
		STableRow::FArguments()
		.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
		.ShowSelection(false),
		InOwnerTableView
	);
}

void SMaterialDynamicParametersOverviewTree::Construct(const FArguments& InArgs)
{
	bHasAnyParameters = false;
	MaterialInstance = InArgs._InMaterialInstance;
	Owner = InArgs._InOwner;
	CreateGroupsWidget();

	STreeView<TSharedPtr<FPropertySortedParamData>>::Construct(
		STreeView::FArguments()
		.TreeItemsSource(&SortedParameters)
		.SelectionMode(ESelectionMode::None)
		.OnGenerateRow(this, &SMaterialDynamicParametersOverviewTree::OnGenerateRowMaterialLayersFunctionsTreeView)
		.OnGetChildren(this, &SMaterialDynamicParametersOverviewTree::OnGetChildrenMaterialLayersFunctionsTreeView)
		.OnExpansionChanged(this, &SMaterialDynamicParametersOverviewTree::OnExpansionChanged)
		.ExternalScrollbar(InArgs._InScrollbar)
	);
}

TSharedRef< ITableRow > SMaterialDynamicParametersOverviewTree::OnGenerateRowMaterialLayersFunctionsTreeView(TSharedPtr<FPropertySortedParamData> Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	TSharedRef< SMaterialDynamicParametersOverviewTreeItem > ReturnRow = SNew(SMaterialDynamicParametersOverviewTreeItem, OwnerTable)
		.StackParameterData(Item)
		.MaterialInstance(MaterialInstance.Get())
		.InTree(SharedThis(this));
	return ReturnRow;
}

void SMaterialDynamicParametersOverviewTree::OnGetChildrenMaterialLayersFunctionsTreeView(TSharedPtr<FPropertySortedParamData> InParent, TArray< TSharedPtr<FPropertySortedParamData> >& OutChildren)
{
	OutChildren = InParent->Children;
}

void SMaterialDynamicParametersOverviewTree::OnExpansionChanged(TSharedPtr<FPropertySortedParamData> Item, bool bIsExpanded)
{
	bool* ExpansionValue = MaterialInstance->ParameterOverviewExpansion.Find(Item->NodeKey);
	if (ExpansionValue == nullptr)
	{
		MaterialInstance->ParameterOverviewExpansion.Add(Item->NodeKey, bIsExpanded);
	}
	else if (*ExpansionValue != bIsExpanded)
	{
		MaterialInstance->ParameterOverviewExpansion.Emplace(Item->NodeKey, bIsExpanded);
	}
	// Expand any children that are also expanded
	for (TSharedPtr<FPropertySortedParamData> Child : Item->Children)
	{
		bool* ChildExpansionValue = MaterialInstance->ParameterOverviewExpansion.Find(Child->NodeKey);
		if (ChildExpansionValue != nullptr && *ChildExpansionValue == true)
		{
			SetItemExpansion(Child, true);
		}
	}
}

void SMaterialDynamicParametersOverviewTree::SetParentsExpansionState()
{
	for (const auto& Pair : SortedParameters)
	{
		if (Pair->Children.Num())
		{
			bool* bIsExpanded = MaterialInstance->ParameterOverviewExpansion.Find(Pair->NodeKey);
			if (bIsExpanded)
			{
				SetItemExpansion(Pair, *bIsExpanded);
			}
			else
			{
				SetItemExpansion(Pair, true);
			}
		}
	}
}

void SMaterialDynamicParametersOverviewTree::CreateGroupsWidget()
{
	check(MaterialInstance.IsValid());
	UnsortedParameters.Reset();
	SortedParameters.Reset();

	const TArray<TSharedRef<IDetailTreeNode>> TestData = GetOwner().Pin()->Generator->GetRootTreeNodes();
	if (TestData.Num() == 0)
	{
		return;
	}
	TSharedPtr<IDetailTreeNode> Category = TestData[0];
	TSharedPtr<IDetailTreeNode> ParameterGroups;
	TArray<TSharedRef<IDetailTreeNode>> Children;
	Category->GetChildren(Children);

	TArray<TSharedPtr<IPropertyHandle>> DeferredSearches;

	for (int32 ChildIdx = 0; ChildIdx < Children.Num(); ChildIdx++)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = Children[ChildIdx]->CreatePropertyHandle();

		bHasAnyParameters = true;

		if (!PropertyHandle.IsValid() || !PropertyHandle->GetProperty())
		{
			continue;
		}

		if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UMaterialInstance, ScalarParameterValues))
		{
			Utilities::AddParameters<FScalarParameterValue>(PropertyHandle,
			                                                 GET_MEMBER_NAME_CHECKED(
				                                                 FScalarParameterValue, ParameterValue),
			                                                 DeferredSearches, UnsortedParameters);
		}
		else if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UMaterialInstance, VectorParameterValues))
		{
			Utilities::AddParameters<FVectorParameterValue>(PropertyHandle,
															GET_MEMBER_NAME_CHECKED(
																FVectorParameterValue, ParameterValue),
															DeferredSearches, UnsortedParameters);
		}
		else if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UMaterialInstance, TextureParameterValues))
		{
			Utilities::AddParameters<FTextureParameterValue>(PropertyHandle,
															GET_MEMBER_NAME_CHECKED(
																FTextureParameterValue, ParameterValue),
															DeferredSearches, UnsortedParameters);
		}
		else if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UMaterialInstance, FontParameterValues))
		{
			Utilities::AddParameters<FFontParameterValue>(PropertyHandle,
															GET_MEMBER_NAME_CHECKED(
																FFontParameterValue, FontValue),
															DeferredSearches, UnsortedParameters);
		}
	}

	checkf(UnsortedParameters.Num() == DeferredSearches.Num(), TEXT("Internal inconsistency: number of node searches does not match the number of properties"));
	TArray<TSharedPtr<IDetailTreeNode>> DeferredResults = GetOwner().Pin()->Generator->FindTreeNodes(DeferredSearches);
	checkf(UnsortedParameters.Num() == DeferredResults.Num(), TEXT("Internal inconsistency: number of node search results does not match the number of properties"));

	for (int Idx = 0, NumUnsorted = UnsortedParameters.Num(); Idx < NumUnsorted; ++Idx)
	{
		FPropertyUnsortedParamData& NonLayerProperty = UnsortedParameters[Idx];
		NonLayerProperty.ParameterNode = DeferredResults[Idx];
		NonLayerProperty.ParameterHandle = NonLayerProperty.ParameterNode->CreatePropertyHandle();
	}

	Children.Empty();

	ShowSubParameters();
	RequestTreeRefresh();
	SetParentsExpansionState();
}

void SMaterialDynamicParametersOverviewTree::ShowSubParameters()
{
	for (FPropertyUnsortedParamData Property : UnsortedParameters)
	{
		TSharedPtr<FPropertySortedParamData> GroupProperty(new FPropertySortedParamData());
		GroupProperty->ParameterInfo = Property.ParameterInfo;
		GroupProperty->StackDataType = EStackDataType::Group;
		GroupProperty->GroupName = Property.GroupName;
		GroupProperty->NodeKey = FString::FromInt(GroupProperty->ParameterInfo.Index) + FString::FromInt(GroupProperty->ParameterInfo.Association) + Property.GroupName.ToString();

		bool bAddNewGroup = true;
		for (TSharedPtr<struct FPropertySortedParamData> GroupChild : SortedParameters)
		{
			if (GroupChild->NodeKey == GroupProperty->NodeKey)
			{
				bAddNewGroup = false;
			}
		}
		if (bAddNewGroup)
		{
			SortedParameters.Add(GroupProperty);
		}

		TSharedPtr<FPropertySortedParamData> ChildProperty(new FPropertySortedParamData());
		ChildProperty->ParameterInfo = Property.ParameterInfo;
		ChildProperty->StackDataType = EStackDataType::Property;
		ChildProperty->ParameterNode = Property.ParameterNode;
		ChildProperty->NodeKey = FString::FromInt(ChildProperty->ParameterInfo.Index) + FString::FromInt(ChildProperty->ParameterInfo.Association) + Property.GroupName.ToString() + Property.ParameterInfo.Name.ToString();

		TArray<TSharedRef<IDetailTreeNode>> ParamChildren;
		Property.ParameterNode->GetChildren(ParamChildren);
		for (int32 ParamChildIdx = 0; ParamChildIdx < ParamChildren.Num(); ParamChildIdx++)
		{
			TSharedPtr<FPropertySortedParamData> ParamChildProperty(new FPropertySortedParamData());
			ParamChildProperty->StackDataType = EStackDataType::PropertyChild;
			ParamChildProperty->ParameterNode = ParamChildren[ParamChildIdx];
			ParamChildProperty->ParameterHandle = ParamChildProperty->ParameterNode->CreatePropertyHandle();
			ParamChildProperty->ParameterInfo = Property.ParameterInfo;
			ChildProperty->Children.Add(ParamChildProperty);
		}

		for (TSharedPtr<struct FPropertySortedParamData> GroupChild : SortedParameters)
		{
			if (GroupChild->GroupName == Property.GroupName
				&& GroupChild->ParameterInfo.Association == ChildProperty->ParameterInfo.Association
				&&  GroupChild->ParameterInfo.Index == ChildProperty->ParameterInfo.Index)
			{
				GroupChild->Children.Add(ChildProperty);
			}
		}
	}
}

void SMaterialDynamicParametersPanelWidget::Refresh()
{
	TSharedPtr<SHorizontalBox> HeaderBox;
	NestedTree->CreateGroupsWidget();

	if (NestedTree->HasAnyParameters())
	{
		this->ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(0.0f)
			[
				
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(HeaderBox, SHorizontalBox)
				]
				+ SVerticalBox::Slot()
				.Padding(0.0f)
				.MaxHeight(300.f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					[
						NestedTree.ToSharedRef()
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.AutoWidth()
					[
						SNew(SBox)
						.WidthOverride(16.0f)
						[
							ExternalScrollbar.ToSharedRef()
						]
					]
					
				]
				
			]
		];

		HeaderBox->AddSlot()
			.FillWidth(1.0f)
			[
				SNullWidget::NullWidget
			];
	}
	else
	{
		this->ChildSlot
			[
				SNew(SBox)
				.Padding(10.0f)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ConnectMaterialParametersToFillList", "Connect a parameter to see it here."))
				]
			];
	}
}

void SMaterialDynamicParametersPanelWidget::Construct(const FArguments& InArgs)
{
	MaterialInstance = InArgs._InMaterialInstance;
	ExternalScrollbar = SNew(SScrollBar);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	const FPropertyRowGeneratorArgs PropertyRowGeneratorArgs;
	Generator = PropertyEditorModule.CreatePropertyRowGenerator(PropertyRowGeneratorArgs);

	Generator->SetObjects({ MaterialInstance.Get() });

	NestedTree = SNew(SMaterialDynamicParametersOverviewTree)
		.InMaterialInstance(InArgs._InMaterialInstance)
		.InOwner(SharedThis(this))
		.InScrollbar(ExternalScrollbar);

	
	Refresh();
}

void SMaterialDynamicParametersPanelWidget::UpdateInstance(UMaterialInstance* InMaterialInstance)
{
	Generator->SetObjects({ InMaterialInstance });
	NestedTree->MaterialInstance = InMaterialInstance;
	Refresh();
}

#undef LOCTEXT_NAMESPACE
