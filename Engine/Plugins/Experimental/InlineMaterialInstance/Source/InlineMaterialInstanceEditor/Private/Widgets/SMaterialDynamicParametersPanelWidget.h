// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IPropertyRowGenerator;
class SMaterialDynamicParametersOverviewTree;
class UMaterialInstance;

/**
 * Dynamic Material parameters view. The view creates all widgets for dynamic material instance parameters
 * Most of the logic copied from SMaterialParametersOverviewTreeItem
 */
class SMaterialDynamicParametersPanelWidget : public SCompoundWidget
{
	friend SMaterialDynamicParametersOverviewTree;
	
public:
	SLATE_BEGIN_ARGS(SMaterialDynamicParametersPanelWidget)
		: _InMaterialInstance(nullptr)
	{}

	SLATE_ARGUMENT(UMaterialInstance*, InMaterialInstance)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Update material instance for tree generation and refresh */
	void UpdateInstance(UMaterialInstance* InMaterialInstance);

private:
	/** Refresh MaterialParameters tree */
	void Refresh();

private:
	/** The set of material parameters this is associated with */
	TWeakObjectPtr<UMaterialInstance> MaterialInstance;

	/** The tree contained in this item */
	TSharedPtr<class SMaterialDynamicParametersOverviewTree> NestedTree;

	/** Optional external Scroll bar */
	TSharedPtr<class SScrollBar> ExternalScrollbar;

	/** Row generator instance */
	TSharedPtr<IPropertyRowGenerator> Generator;
};

