// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMaterialDynamicWidgets.h"

#include "Components/ActorComponent.h"
#include "Editor.h"
#include "MaterialEditingLibrary.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "PropertyCustomizationHelpers.h"
#include "SMaterialDynamicParametersPanelWidget.h"
#include "ScopedTransaction.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FMaterialDynamicList"

void SMaterialDynamicView::Construct(const FArguments& InArgs, const TSharedRef<FMaterialItemView>& InMaterialItemView, UActorComponent* InCurrentComponent)
{
	MaterialItemViewWeakPtr = InMaterialItemView;
	CurrentComponent = InCurrentComponent;

	TSharedRef<SVerticalBox> ResultWidget = SNew(SVerticalBox);
	if (InCurrentComponent != nullptr)
	{
		UMaterialInterface* MaterialInterface = InMaterialItemView->GetMaterialListItem().Material.Get();
		UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialInterface);
		UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<UMaterialInstanceDynamic>(MaterialInterface);
		
		if (MaterialInstance)
		{
			const bool bIsInvertedVisibility = !!MaterialInstanceDynamic;
			const TAttribute<EVisibility> CreateDynamicMaterialButtonVisibility = GetButtonVisibilityAttribute<UMaterialInstance>(bIsInvertedVisibility);
			
			ResultWidget->AddSlot()
			.AutoHeight()
			[
				SNew(SBox)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.Visibility(GetButtonVisibilityAttribute<UMaterialInstanceDynamic>())
						.ToolTipText(LOCTEXT("Revert_Button_Tooltip", "Revert the Dynamic Material Instance back to the original Material Instance"))
						.OnClicked(this, &SMaterialDynamicView::OnRevertButtonClicked)
						[
							SNew(STextBlock).Text(LOCTEXT("Revert_Button", "Revert"))
						]
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.Visibility(GetButtonVisibilityAttribute<UMaterialInstanceDynamic>())
						.ToolTipText(LOCTEXT("Reset_Button_Tooltip", "Reset the properties to the Material Instance default"))
						.OnClicked(this, &SMaterialDynamicView::OnResetButtonClicked)
						[
							SNew(STextBlock).Text(LOCTEXT("Reset_Button", "Reset"))
						]
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.Visibility(GetButtonVisibilityAttribute<UMaterialInstanceDynamic>())
						.ToolTipText(LOCTEXT("CopyToOriginal_Button_Tooltip", "Copy and overwrite the parameters onto the original Material Instance"))
						.OnClicked(this, &SMaterialDynamicView::OnCopyToOriginalButtonClicked)
						[
							SNew(STextBlock).Text(LOCTEXT("CopyToOriginal_Button", "Copy to Original"))
						]
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.Visibility(CreateDynamicMaterialButtonVisibility)
						.ToolTipText(LOCTEXT("CreateDynamicMaterial_Button_Tooltip", "Create a Dynamic Material Instance for this Material Instance and automatically sets it"))
						.OnClicked(this, &SMaterialDynamicView::OnCreateDynamicMaterialButtonClicked)
						[
							SNew(STextBlock).Text(LOCTEXT("CreateDynamicMaterial_Button", "Create Dynamic Material"))
						]
					]
				]
			];
		}

		if (MaterialInstanceDynamic)
		{
			ResultWidget->AddSlot()
			.AutoHeight()
			[
				SNew(SMaterialDynamicParametersPanelWidget)
				.InMaterialInstance(MaterialInstanceDynamic)
			];
		}
	}

	ChildSlot
	[
		ResultWidget
	];
}

FReply SMaterialDynamicView::OnResetButtonClicked() const
{
	UActorComponent* OwnerActorComponent = CurrentComponent.Get();
	if (!ensure(OwnerActorComponent))
	{
		return FReply::Handled();
	}

	const TSharedPtr<FMaterialItemView> MaterialItemView = MaterialItemViewWeakPtr.Pin();
	if(!ensure(MaterialItemView.IsValid()))
	{
		return FReply::Handled();
	}
	
	UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<UMaterialInstanceDynamic>(MaterialItemView->GetMaterialListItem().Material.Get());
	if (!MaterialInstanceDynamic)
	{
		return FReply::Handled();
	}

	UMaterialInstance* ParentMaterialInstance = Cast<UMaterialInstance>(MaterialInstanceDynamic->Parent);
	if (!ParentMaterialInstance)
	{
		return FReply::Handled();
	}


	FScopedTransaction Transaction(LOCTEXT("ResetDynamicMaterial", "Reset Dynamic Material"));
	MaterialInstanceDynamic->Modify();
	OwnerActorComponent->Modify();
	MaterialInstanceDynamic->CopyParameterOverrides(ParentMaterialInstance);

	return FReply::Handled();
}

FReply SMaterialDynamicView::OnRevertButtonClicked() const
{
	UActorComponent* OwnerActorComponent = CurrentComponent.Get();
	if (!ensure(OwnerActorComponent))
	{
		return FReply::Handled();
	}

	const TSharedPtr<FMaterialItemView> MaterialItemView = MaterialItemViewWeakPtr.Pin();
	if(!ensure(MaterialItemView.IsValid()))
	{
		return FReply::Handled();
	}
	
	UMaterialInstance* ParentMaterialInstance = GetMaterialParent<UMaterialInstance, UMaterialInstanceDynamic>(MaterialItemView->GetMaterialListItem().Material.Get());
	if (!ParentMaterialInstance)
	{
		return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("RevertDynamicMaterial", "Revert Dynamic Material"));
	MaterialItemView->ReplaceMaterial(ParentMaterialInstance);
	
	return FReply::Handled();
}

FReply SMaterialDynamicView::OnCopyToOriginalButtonClicked() const
{
	UActorComponent* OwnerActorComponent = CurrentComponent.Get();
	if (!ensure(OwnerActorComponent))
	{
		return FReply::Handled();
	}

	const TSharedPtr<FMaterialItemView> MaterialItemView = MaterialItemViewWeakPtr.Pin();
	if(!ensure(MaterialItemView.IsValid()))
	{
		return FReply::Handled();
	}
	
	UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<UMaterialInstanceDynamic>(MaterialItemView->GetMaterialListItem().Material.Get());
	if (!MaterialInstanceDynamic)
	{
		return FReply::Handled();
	}

	UMaterialInstance* ParentMaterialInstance = Cast<UMaterialInstance>(MaterialInstanceDynamic->Parent);
	if (!ParentMaterialInstance)
	{
		return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("CopyToOriginalMaterial", "Copy To Original Material"));
	ParentMaterialInstance->ScalarParameterValues = MaterialInstanceDynamic->ScalarParameterValues;
	ParentMaterialInstance->VectorParameterValues = MaterialInstanceDynamic->VectorParameterValues;
	ParentMaterialInstance->TextureParameterValues = MaterialInstanceDynamic->TextureParameterValues;
	ParentMaterialInstance->RuntimeVirtualTextureParameterValues = MaterialInstanceDynamic->RuntimeVirtualTextureParameterValues;
	ParentMaterialInstance->FontParameterValues = MaterialInstanceDynamic->FontParameterValues;
	ParentMaterialInstance->UpdateStaticPermutation();
	ParentMaterialInstance->Modify();

#if WITH_EDITOR
	// Check if the Parent is being Edited actively in a Window somewhere
	if (GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(ParentMaterialInstance, false))
	{
		UMaterialEditingLibrary::RecompileMaterial(ParentMaterialInstance->GetMaterial());
	}
#endif

	return FReply::Handled();
}

FReply SMaterialDynamicView::OnCreateDynamicMaterialButtonClicked() const
{
	const TSharedPtr<FMaterialItemView> MaterialItemView = MaterialItemViewWeakPtr.Pin();
	if(!ensure(MaterialItemView.IsValid()))
	{
		return FReply::Handled();
	}
	
	UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialItemView->GetMaterialListItem().Material.Get());
	UActorComponent* OwnerActorComponent = CurrentComponent.Get();

	if (!OwnerActorComponent)
	{
		return FReply::Handled();
	}

	if (MaterialInstance)
	{
		OwnerActorComponent->Modify();

		FScopedTransaction Transaction(LOCTEXT("CreateDynamicMaterial", "Create Dynamic Material"));

		// Create Transactional Material. That is allawed create MID with MU and transact the changes
		UMaterialInstanceDynamic* NewMaterialInstanceDynamic = NewObject<UMaterialInstanceDynamic>(OwnerActorComponent, NAME_None, RF_Public | RF_Transactional);
		NewMaterialInstanceDynamic->Parent = MaterialInstance;
		NewMaterialInstanceDynamic->UpdateCachedData();


		// Copy Parameter Overrides with Transaction
		NewMaterialInstanceDynamic->CopyParameterOverrides(MaterialInstance);
		MaterialItemView->ReplaceMaterial(NewMaterialInstanceDynamic);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE

