// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MaterialList.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;
class IDetailLayoutBuilder;
class UActorComponent;
class UMaterialInterface;

/**
 * Creates the view for material dynamic view
 */
class SMaterialDynamicView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMaterialDynamicView)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FMaterialItemView>& InMaterialItemView, UActorComponent* InCurrentComponent);

private:
	/** Get Visibility for extension button */
	template<typename TMaterialClass>
	TAttribute<EVisibility> GetButtonVisibilityAttribute(const bool bIsInvertedResult = false) const
	{
		return MakeAttributeLambda([this, bIsInvertedResult]()
			{
				const TSharedPtr<FMaterialItemView> MaterialItemView = MaterialItemViewWeakPtr.Pin();
				if(!ensure(MaterialItemView.IsValid()))
				{
					return EVisibility::Collapsed;
				}
			
				bool IsVisible = Cast<TMaterialClass>(MaterialItemView->GetMaterialListItem().Material.Get()) ? true : false;
				IsVisible ^= bIsInvertedResult;

				return IsVisible ? EVisibility::Visible : EVisibility::Collapsed;
			});
	};

	/** Button handler for revert to original material for the slot */
	FReply OnRevertButtonClicked() const;

	/** Button handler for reset parameters to default instance material parameter */
	FReply OnResetButtonClicked() const;

	/** Button handler for copy material parameters to original instance material */
	FReply OnCopyToOriginalButtonClicked() const;

	/** Button handler for create dynamic material */
	FReply OnCreateDynamicMaterialButtonClicked() const;

	/** Get parent material and cast to specified type */
	template<typename TReturnType, typename TInputType>
	TReturnType* GetMaterialParent(UMaterialInterface* InMaterialInterface) const
	{
		if (TInputType* Material = Cast<TInputType>(InMaterialInterface))
		{
			return Cast<TReturnType>(Material->Parent);
		}

		return nullptr;
	}

private:
	/** Weak Pointer to Material Item View */
	TWeakPtr<FMaterialItemView> MaterialItemViewWeakPtr;

	/** Weak pointer to current component using the current material */
	TWeakObjectPtr<UActorComponent> CurrentComponent;
};

